{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Concert Shared Slate",
	"Description": "Contains UI that is shared by client UI modules only",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"Hidden": true,
	"IsBetaVersion": true,
	"Installed": false,
	"SupportedPrograms": [ "UnrealMultiUserServer", "UnrealRecoverySvc", "CrashReportClientEditor" ],
	"Modules": [
        {
          "Name": "ConcertClientSharedSlate",
          "Type": "UncookedOnly",
          "LoadingPhase": "Default",
          "ProgramAllowList": [ "UnrealRecoverySvc" ]
        }
	],
	"Plugins": [
		{
			"Name": "ConcertMain",
			"Enabled": true
		},
		{
			"Name": "ConcertSharedSlate",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncCore",
			"Enabled": true
		},
        {
          "Name": "ConcertSyncClient",
          "Enabled": true
        }
	]
}
