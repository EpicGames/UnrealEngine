// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ConcertClientSharedSlate : ModuleRules
	{
		public ConcertClientSharedSlate(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"EditorStyle",
					
					// Concert
					"ConcertSharedSlate",
					"ConcertSyncClient",
				}
			);
			
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					// Engine
					"AssetDefinition",
					"AssetRegistry", 
					"EditorWidgets",
					"InputCore",
					"Projects",
					"Slate",
					"SlateCore",
					"SubobjectDataInterface",
					"ToolMenus",
					"ToolWidgets",
					"UnrealEd", 
					
					// Concert
					"Concert",
					"ConcertClient",
					"ConcertSyncCore",
					"ConcertTransport", // For LogConcert
				}
			);

			ShortName = "ConClShrSlt";
		}
	}
}
