// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/ClientReplicationWidgetFactories.h"

#include "Editor/View/SReplicationDropArea.h"
#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Editor/Model/Object/EditorObjectHierarchyModel.h"
#include "Replication/Editor/Model/Object/EditorObjectNameModel.h"
#include "Replication/Editor/Model/ReplicationStreamObject.h"
#include "Replication/Editor/Model/TransactionalReplicationStreamModel.h"
#include "Replication/Editor/View/PropertyTree/SFilteredPropertyTreeView.h"

#include "UObject/UObjectGlobals.h"
#include "UObject/Package.h"

namespace UE::ConcertClientSharedSlate
{
	TSharedRef<ConcertSharedSlate::IObjectHierarchyModel> CreateObjectHierarchyForComponentHierarchy()
	{
		return MakeShared<FEditorObjectHierarchyModel>();
	}

	TSharedRef<ConcertSharedSlate::IObjectNameModel> CreateEditorObjectNameModel()
	{
		return MakeShared<FEditorObjectNameModel>();
	}

	TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> CreateTransactionalStreamModel(
		const TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel>& BaseModel,
		UObject& OwnerObject
		)
	{
		return MakeShared<ConcertSharedSlate::FTransactionalReplicationStreamModel>(
			BaseModel,
			OwnerObject
			);
	}

	TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> CreateTransactionalStreamModel()
	{
		constexpr EObjectFlags Flags = RF_Transient | RF_Transactional;
		UReplicationStreamObject* Object = NewObject<UReplicationStreamObject>(GetTransientPackage(), NAME_None, Flags);

		TAttribute<FConcertObjectReplicationMap*> Attribute = TAttribute<FConcertObjectReplicationMap*>::CreateLambda([WeakPtr = TWeakObjectPtr<UReplicationStreamObject>(Object)]() -> FConcertObjectReplicationMap* 
		{
			if (WeakPtr.IsValid())
			{
				return &WeakPtr->ReplicationMap;
			}
			return nullptr;
		});

		return CreateTransactionalStreamModel(ConcertSharedSlate::CreateBaseStreamModel(MoveTemp(Attribute)), *Object);
	}
	
	TSharedRef<ConcertSharedSlate::IPropertyTreeView> CreateFilterablePropertyTreeView(FFilterablePropertyTreeViewParams Params)
	{
		return SNew(SFilteredPropertyTreeView, MoveTemp(Params));
	}

	ConcertSharedSlate::FWrapOutlinerWidget CreateDropTargetOutlinerWrapper(FCreateDropTargetOutlinerWrapperParams Params)
	{
		return ConcertSharedSlate::FWrapOutlinerWidget::CreateLambda([Params = MoveTemp(Params)](const TSharedRef<SWidget>& Widget)
		{
			return SNew(SReplicationDropArea)
				.HandleDroppedObjects(Params.HandleDroppedObjectsDelegate)
				[
					Widget
				];
		});
	}
}
