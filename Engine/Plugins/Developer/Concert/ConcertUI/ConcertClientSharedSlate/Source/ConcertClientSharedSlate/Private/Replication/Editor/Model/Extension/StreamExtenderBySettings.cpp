// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/Extension/StreamExtenderBySettings.h"

#include "Replication/Data/ConcertPropertySelection.h"
#include "Replication/Editor/Model/Extension/IStreamExtensionContext.h"
#include "Replication/Settings/ConcertStreamObjectAutoBindingRules.h"

namespace UE::ConcertClientSharedSlate
{
	FStreamExtenderBySettings::FStreamExtenderBySettings(TAttribute<const FConcertStreamObjectAutoBindingRules*> InReplicationSettingsAttribute)
		: ReplicationSettingsAttribute(MoveTemp(InReplicationSettingsAttribute))
	{}

	void FStreamExtenderBySettings::ExtendStream(UObject& ExtendedObject, ConcertSharedSlate::IStreamExtensionContext& Context)
	{
		const FConcertStreamObjectAutoBindingRules* Settings = ReplicationSettingsAttribute.Get();
		if (!ensure(Settings))
		{
			return;
		}
		Settings->AddDefaultPropertiesFromSettings(*ExtendedObject.GetClass(), [&ExtendedObject, &Context](FConcertPropertyChain&& PropertyChain)
		{
			Context.AddPropertyTo(ExtendedObject, MoveTemp(PropertyChain));
		});
		Settings->DefaultAddedSubobjectRules.MatchSubobjectsRecursively(ExtendedObject, [&ExtendedObject, &Context](UObject& AdditionalObject)
		{
			Context.AddAdditionalObject(AdditionalObject);
		});
	}
}
