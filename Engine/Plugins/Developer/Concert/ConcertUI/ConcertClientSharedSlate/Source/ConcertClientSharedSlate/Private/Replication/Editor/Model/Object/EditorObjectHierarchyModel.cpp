// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditorObjectHierarchyModel.h"

#include "Components/SceneComponent.h"
#include "Misc/EBreakBehavior.h"

#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectHash.h"

namespace UE::ConcertClientSharedSlate
{
	namespace Private
	{
		static bool IsValidComponent(const UActorComponent& Component)
		{
			return !Component.IsVisualizationComponent();
		}

		static void DiscoverComponents(
			const UObject& Object,
			const TFunctionRef<EBreakBehavior(const TSoftObjectPtr<>& Child, ConcertSharedSlate::EChildRelationship Relationship)>& Callback
			)
		{
			using namespace ConcertSharedSlate;
			
			if (const AActor* AsActor = Cast<AActor>(&Object))
			{
				// Only the root component should be listed. It's children should be returned through recursive ForEachDirectChild calls.
				USceneComponent* RootComponent = AsActor->GetRootComponent();
				if (IsValid(RootComponent) && IsValidComponent(*RootComponent) && Callback(RootComponent, EChildRelationship::Component) == EBreakBehavior::Break)
				{
					return;
				}

				for (UActorComponent* Component : TInlineComponentArray<UActorComponent*>(AsActor))
				{
					if (IsValid(Component)
						&& IsValidComponent(*Component)
						// The only component to be found here should be the root component, the others are found via recursive ForEachDirectChild calls
						&& !Component->IsA<USceneComponent>()
						&& Callback(Component, EChildRelationship::Component) == EBreakBehavior::Break)
					{
						break;
					}
				}
			}
		
			if (const USceneComponent* SceneComponent = Cast<USceneComponent>(&Object))
			{
				for (int32 i = 0; i < SceneComponent->GetNumChildrenComponents(); ++i)
				{
					USceneComponent* ChildComponent = SceneComponent->GetChildComponent(i);
					if (IsValid(ChildComponent) // GetChildComponent can return nullptr sometimes...
						&& IsValidComponent(*ChildComponent)
						&& Callback(ChildComponent, EChildRelationship::Component) == EBreakBehavior::Break)
					{
						break;
					}
				}
			}
		}
	}

	void FEditorObjectHierarchyModel::ForEachDirectChild(
		const TSoftObjectPtr<>& Parent,
		TFunctionRef<EBreakBehavior(const TSoftObjectPtr<>& Object, ConcertSharedSlate::EChildRelationship Relationship)> Callback,
		ConcertSharedSlate::EChildRelationshipFlags InclusionFlags
		) const
	{
		using namespace ConcertSharedSlate;

		// In the context of replication, only resolve objects but do not load them. Case: client 1 may be viewing objects of client 2 but they are in different worlds.
		const UObject* Object = Parent.Get();
		if (!Object)
		{
			return;
		}

		if (EnumHasAnyFlags(InclusionFlags, EChildRelationshipFlags::Component))
		{
			Private::DiscoverComponents(*Object, Callback);
		}
		
		if (EnumHasAnyFlags(InclusionFlags, EChildRelationshipFlags::Subobject))
		{
			ForEachObjectWithOuterBreakable(Object, [&Callback](UObject* Subobject)
			{
				// Components were already handled above - do not double report.
				const bool bContinueIteration = Subobject->IsA<UActorComponent>() || Callback(Subobject, EChildRelationship::Subobject) == EBreakBehavior::Continue;
				return bContinueIteration;
			}, EGetObjectsFlags::None);
		}
	}

	TOptional<ConcertSharedSlate::IObjectHierarchyModel::FParentInfo> FEditorObjectHierarchyModel::GetParentInfo(const TSoftObjectPtr<const UObject>& ChildObject) const
	{
		using namespace ConcertSharedSlate;

		// In the context of replication, only resolve objects but do not load them. Case: client 1 may be viewing objects of client 2 but they are in different worlds.
		const UObject* Object = ChildObject.Get();
		if (!Object)
		{
			return {};
		}

		if (Object->IsA<AActor>()
			// Unlikely but theoretically possible that it's an asset or some transient object.
			|| !Object->IsInA(AActor::StaticClass()))
		{
			return {};
		}
		
		const UObject* Outer = Object->GetOuter();
		const EChildRelationship Relationship = Object->IsA<UActorComponent>() ? EChildRelationship::Component : EChildRelationship::Subobject;
		return FParentInfo{ Outer, Relationship };
	}
}
