// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Object/IObjectHierarchyModel.h"
#include "UObject/SoftObjectPtr.h"

class AActor;

namespace UE::ConcertClientSharedSlate
{
	/**
	 * Builds a similar tree hierarchy as SSubobjectEditor with the addition of non-component subobjects.
	 *
	 * An example for a hierarchy in an editor build is:
	 * - Actor returns its root component, non-scene components (UActorComponents), and all other direct subobjects
	 * - Scene components return their children, and all other direct subobjects
	 * - All other subobjects report all direct subobjects
	 */
	class FEditorObjectHierarchyModel : public ConcertSharedSlate::IObjectHierarchyModel
	{
	public:

		//~ Begin IObjectHierarchy Interface
		virtual void ForEachDirectChild(
			const TSoftObjectPtr<>& Parent,
			TFunctionRef<EBreakBehavior(const TSoftObjectPtr<>& Object, ConcertSharedSlate::EChildRelationship Relationship)> Callback,
			ConcertSharedSlate::EChildRelationshipFlags InclusionFlags
			) const override;
		virtual TOptional<FParentInfo> GetParentInfo(const TSoftObjectPtr<const UObject>& ChildObject) const override;
		//~ End IObjectHierarchy Interface
	};
}


