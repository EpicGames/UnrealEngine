// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/ObjectNameUtils.h"
#include "Replication/Editor/Model/Object/IObjectNameModel.h"

namespace UE::ConcertClientSharedSlate
{
	/** Name model that uses editor data for determining display names: actors use their labels, components ask USubobjectDataSubsystem.  */
	class FEditorObjectNameModel : public ConcertSharedSlate::IObjectNameModel
	{
	public:
		
		//~ Begin IObjectNameModel Interface
		virtual FText GetObjectDisplayName(const TSoftObjectPtr<>& ObjectPath) const override { return ConcertClientSharedSlate::GetObjectDisplayName(ObjectPath); }
		//~ End IObjectNameModel Interface
	};
}

