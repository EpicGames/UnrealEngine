// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/ObjectSource/ActorSelectionSourceModel.h"

#include "Replication/Editor/Model/ObjectSource/SelectedActorsSource.h"
#include "Replication/Editor/Model/ObjectSource/WorldActorSource.h"

#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "FEditorObjectSelectionSourceModel"

namespace UE::ConcertClientSharedSlate
{
	FActorSelectionSourceModel::FActorSelectionSourceModel()
		: BaseActorCategory([this]()
		{
			return ConcertSharedSlate::FObjectSourceCategory
			{
				ConcertSharedSlate::FBaseDisplayInfo
				{
					LOCTEXT("ObjectsCategory.Label", "Add"),
					LOCTEXT("ObjectsCategory.Tooltip", "Add objects from the open editor world")
				},
				{
					MakeShared<FSelectedActorsSource>(),
					MakeShared<FWorldActorSource>()
				}
			};
		}())
	{}

	TArray<ConcertSharedSlate::FObjectSourceCategory> FActorSelectionSourceModel::GetRootSources() const
	{
		ConcertSharedSlate::FObjectSourceCategory ActorCategory = BaseActorCategory;
		return { ActorCategory };
	}

	TArray<TSharedRef<ConcertSharedSlate::IObjectSourceModel>> FActorSelectionSourceModel::GetContextMenuOptions(const FSoftObjectPath& Item)
	{
		return {};
	}
}

#undef LOCTEXT_NAMESPACE