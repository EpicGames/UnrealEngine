// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/ObjectSource/SelectedActorsSource.h"

#include "Editor.h"
#include "Selection.h"

#define LOCTEXT_NAMESPACE "FSelectedActorsSourceModel"

namespace UE::ConcertClientSharedSlate
{
	ConcertSharedSlate::FSourceDisplayInfo FSelectedActorsSource::GetDisplayInfo() const
	{
		return
		{
			{
				LOCTEXT("Label", "Selected Actors"),
			   LOCTEXT("Tooltip", "Add the actors under your current selection in the world outliner"),
			   FSlateIcon{}
			},
			ConcertSharedSlate::ESourceType::AddOnClick
		};
	}

	uint32 FSelectedActorsSource::GetNumSelectableItems() const
	{
		return ensure(GEditor)
			? GEditor->GetSelectedActors()->Num()
			: 0;
	}

	void FSelectedActorsSource::EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FSelectableObjectInfo& SelectableOption)> Delegate) const
	{
		if (!ensure(GEditor))
		{
			return;
		}

		USelection* SelectedActors = GEditor->GetSelectedActors();
		for (int32 i = 0; i < SelectedActors->Num(); ++i)
		{
			UObject* SelectedActor = SelectedActors->GetSelectedObject(i);
			if (SelectedActor && Delegate({ *SelectedActor }) == EBreakBehavior::Break)
			{
				return;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE