// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/ObjectSource/WorldActorSource.h"

#include "Algo/Accumulate.h"
#include "Algo/Count.h"
#include "CoreGlobals.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "FWorldActorSource"

namespace UE::ConcertClientSharedSlate
{
	ConcertSharedSlate::FSourceDisplayInfo FWorldActorSource::GetDisplayInfo() const
	{
		return {{
			LOCTEXT("Label", "Add Actor from World"),
		   LOCTEXT("Tooltip", "Pick an actor from the editor world")
		}};
	}

	uint32 FWorldActorSource::GetNumSelectableItems() const
	{
		return ensure(GWorld)
			? Algo::TransformAccumulate(GWorld->GetLevels(), [](ULevel* Level)
			{
				return Level
					? Algo::CountIf(Level->Actors, [](AActor* Actor){ return Actor && Actor->IsListedInSceneOutliner(); })
					: 0;
			}, 0)
			: 0;
	}

	void FWorldActorSource::EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FSelectableObjectInfo& SelectableOption)> Delegate) const
	{
		if (!ensure(GWorld))
		{
			return;
		}

		for (TActorIterator<AActor> ActorIt(GWorld); ActorIt; ++ActorIt)
		{
			AActor* Actor = *ActorIt;
			if (Actor && Actor->IsListedInSceneOutliner() && Delegate({ *Actor }) == EBreakBehavior::Break)
			{
				return;
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE