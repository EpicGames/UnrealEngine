// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/PropertySource/ConcertSyncCoreReplicatedPropertySource.h"

#include "ConcertLogGlobal.h"
#include "Replication/PropertyChainUtils.h"
#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "FConcertSyncCoreReplicatedPropertySource"

namespace UE::ConcertClientSharedSlate
{
	void FConcertSyncCoreReplicatedPropertySource::SetClass(UClass* InClass)
	{
		Class = InClass;
		NumProperties = 0;
		if (!Class.IsValid())
		{
			return;
		}
		
		ConcertSyncCore::PropertyChain::ForEachReplicatableProperty(*Class.Get(), [this](const auto&, const auto&)
		{
			++NumProperties;
			return EBreakBehavior::Continue;
		});
	}

	ConcertSharedSlate::FSourceDisplayInfo FConcertSyncCoreReplicatedPropertySource::GetDisplayInfo() const
	{
		return {
			{
				LOCTEXT("Label", "Add Property"),
			   FText::GetEmpty(),
			   FSlateIcon()
			},
			ConcertSharedSlate::ESourceType::ShowAsList
		};
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FConcertSyncCoreReplicatedPropertySource::EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FSelectablePropertyInfo& SelectableOption)> Delegate) const
	{
		if (!Class.IsValid())
		{
			return;
		}

		ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*Class.Get(), [this, &Delegate](FConcertPropertyChain&& Property)
		{
			return Delegate({ MoveTemp(Property) });
		});
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

#undef LOCTEXT_NAMESPACE