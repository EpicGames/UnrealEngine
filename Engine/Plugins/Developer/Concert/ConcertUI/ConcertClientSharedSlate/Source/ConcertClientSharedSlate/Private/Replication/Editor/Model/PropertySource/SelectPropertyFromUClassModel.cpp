// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/PropertySource/SelectPropertyFromUClassModel.h"

#include "ConcertLogGlobal.h"
#include "Replication/Editor/Model/PropertySource/ReplicatablePropertySource.h"

namespace UE::ConcertClientSharedSlate
{
	void FSelectPropertyFromUClassModel::ProcessPropertySource(
		const ConcertSharedSlate::FPropertySourceContext& Context,
		TFunctionRef<void(const ConcertSharedSlate::IPropertySource& Model)> Processor
		) const
	{
		UClass* LoadedClass = Context.Class.TryLoadClass<UObject>();
		UE_CLOGF(!LoadedClass, LogConcert, Warning, "Could not resolve class %ls. Properties will not be available.", *Context.Class.ToString());
		
		FReplicatablePropertySource PropertySource(LoadedClass);
		Processor(PropertySource);
	}
}