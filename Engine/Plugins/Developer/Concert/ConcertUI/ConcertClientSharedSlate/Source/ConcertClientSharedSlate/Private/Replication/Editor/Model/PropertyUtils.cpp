// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/Model/PropertyUtils.h"

#include "Algo/AnyOf.h"
#include "Replication/PropertyChainUtils.h"

namespace UE::ConcertClientSharedSlate::PropertyUtils
{
	void EnumerateAdditionalPropertiesToAdd(
		const UClass& ObjectClass,
		TConstArrayView<FConcertPropertyChain> PropertiesToAdd,
		TFunctionRef<void(FConcertPropertyChain&& AdditionalProperty)> Callback
		)
	{
		ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(ObjectClass, [&PropertiesToAdd, &Callback](FConcertPropertyChain&& ConsideredProperty)
		{
			const bool bIsChildOfAddedProperty = Algo::AnyOf(PropertiesToAdd, [&ConsideredProperty](const FConcertPropertyChain& AddedProperty)
			{
				return ConsideredProperty.IsChildOf(AddedProperty);
			});
			if (bIsChildOfAddedProperty)
			{
				Callback(MoveTemp(ConsideredProperty));
			}
			
			return EBreakBehavior::Continue;
		});
	}
}
