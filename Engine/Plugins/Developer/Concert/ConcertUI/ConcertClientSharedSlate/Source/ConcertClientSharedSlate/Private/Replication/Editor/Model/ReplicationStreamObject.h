// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ObjectReplicationMap.h"
#include "UObject/Object.h"
#include "ReplicationStreamObject.generated.h"

/** UObject wrapper to allow transactions on FConcertObjectReplicationMap. */
UCLASS()
class UReplicationStreamObject : public UObject
{
	GENERATED_BODY()
public:
	
	UPROPERTY()
	FConcertObjectReplicationMap ReplicationMap;
};