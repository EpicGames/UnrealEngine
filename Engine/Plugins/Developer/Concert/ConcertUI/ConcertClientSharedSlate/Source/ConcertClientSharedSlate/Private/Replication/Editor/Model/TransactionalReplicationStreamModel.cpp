// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransactionalReplicationStreamModel.h"

#include "ScopedTransaction.h"
#include "Misc/TransactionObjectEvent.h"

#define LOCTEXT_NAMESPACE "FTransactionalReplicationStreamModel"

namespace UE::ConcertSharedSlate
{
	FTransactionalReplicationStreamModel::FTransactionalReplicationStreamModel(
		const TSharedRef<IEditableReplicationStreamModel>& WrappedModel,
		UObject& OwningObject
		)
		: FEditableStreamModelProxy(WrappedModel)
		, OwningObject(&OwningObject)
	{}

	void FTransactionalReplicationStreamModel::AddObjects(TConstArrayView<UObject*> Objects)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddObjects", "Add replicated objects"));
		OwningObject->Modify();
		GetWrappedModel()->AddObjects(Objects);
	}

	void FTransactionalReplicationStreamModel::RemoveObjects(TConstArrayView<FSoftObjectPath> Objects)
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveObjects", "Remove replicated objects"));
		OwningObject->Modify();
		GetWrappedModel()->RemoveObjects(Objects);
	}

	void FTransactionalReplicationStreamModel::AddProperties(const FSoftObjectPath& SoftObjectPath, TConstArrayView<FConcertPropertyChain> Properties)
	{
		const FScopedTransaction Transaction(LOCTEXT("AddProperties", "Add replicated properties"));
		OwningObject->Modify();
		GetWrappedModel()->AddProperties(SoftObjectPath, Properties);
	}

	void FTransactionalReplicationStreamModel::RemoveProperties(const FSoftObjectPath& SoftObjectPath, TConstArrayView<FConcertPropertyChain> Properties)
	{
		const FScopedTransaction Transaction(LOCTEXT("RemoveProperties", "Remove replicated properties"));
		OwningObject->Modify();
		GetWrappedModel()->RemoveProperties(SoftObjectPath, Properties);
	}

	bool FTransactionalReplicationStreamModel::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
	{
		if (!OwningObject.IsValid())
		{
			return false;
		}
		
		for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjectContexts)
		{
			UObject* Object = TransactionObjectPair.Key;
			while (Object != nullptr)
			{
				if (Object == OwningObject)
				{
					return true;
				}
				Object = Object->GetOuter();
			}
		}

		return false;
	}

	void FTransactionalReplicationStreamModel::PostUndo(bool bSuccess)
	{
		OnObjectsChanged().Broadcast({}, {}, EReplicatedObjectChangeReason::ExternalChange);
		OnPropertiesChanged().Broadcast();
	}

	void FTransactionalReplicationStreamModel::PostRedo(bool bSuccess)
	{
		OnObjectsChanged().Broadcast({}, {}, EReplicatedObjectChangeReason::ExternalChange);
		OnPropertiesChanged().Broadcast();
	}

	void FTransactionalReplicationStreamModel::AddReferencedObjects(FReferenceCollector& Collector)
	{
		Collector.AddReferencedObject(OwningObject);
	}
}

#undef LOCTEXT_NAMESPACE