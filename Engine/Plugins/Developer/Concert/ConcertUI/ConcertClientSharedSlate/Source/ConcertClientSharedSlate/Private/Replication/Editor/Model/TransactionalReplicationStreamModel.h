// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Util/EditableStreamModelProxy.h"

#include "EditorUndoClient.h"
#include "UObject/GCObject.h"

struct FConcertObjectReplicationMap;

namespace UE::ConcertSharedSlate
{
	class IStreamExtender;

	/** Special case of FGenericPropertySelectionModel where the edited FConcertObjectReplicationMap lives in an UObject that is RF_Transactional. */
	class FTransactionalReplicationStreamModel
		: public FEditableStreamModelProxy
		, public FSelfRegisteringEditorUndoClient
		, public FGCObject
	{
	public:

		FTransactionalReplicationStreamModel(
			const TSharedRef<IEditableReplicationStreamModel>& WrappedModel,
			UObject& OwningObject
			);
		
		//~ Begin IEditableReplicationStreamModel Interface
		virtual void AddObjects(TConstArrayView<UObject*> Objects) override;
		virtual void RemoveObjects(TConstArrayView<FSoftObjectPath> Objects) override;
		virtual void AddProperties(const FSoftObjectPath&, TConstArrayView<FConcertPropertyChain> Properties) override;
		virtual void RemoveProperties(const FSoftObjectPath&, TConstArrayView<FConcertPropertyChain> Properties) override;
		//~ End IEditableReplicationStreamModel Interface

		//~ Begin FEditorUndoClient Interface
		virtual bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
		virtual void PostUndo(bool bSuccess) override;
		virtual void PostRedo(bool bSuccess) override;
		//~ End FEditorUndoClient Interface

		//~ Begin FGCObject Interface
		virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
		virtual FString GetReferencerName() const override { return TEXT("FTransactionalReplicationStreamModel"); }
		//~ End FGCObject Interface

	private:
		
		/** User of FTransactionalPropertySelectionModel is responsible for keeping OwningObject alive, e.g. via an asset editor. */
		TWeakObjectPtr<UObject> OwningObject;
	};
}


