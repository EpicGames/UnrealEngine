// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/UnrealEditor/HideObjectsNotInWorldLogic.h"

#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "UObject/SoftObjectPath.h"

namespace UE::ConcertClientSharedSlate
{
	FHideObjectsNotInWorldLogic::FHideObjectsNotInWorldLogic()
	{
		if (ensure(GEngine))
		{
			GEngine->OnWorldAdded().AddRaw(this, &FHideObjectsNotInWorldLogic::OnWorldAdded);
			GEngine->OnWorldDestroyed().AddRaw(this, &FHideObjectsNotInWorldLogic::OnWorldDestroyed);
		}
	}

	FHideObjectsNotInWorldLogic::~FHideObjectsNotInWorldLogic()
	{
		if (GEngine)
		{
			GEngine->OnWorldAdded().RemoveAll(this);
			GEngine->OnWorldDestroyed().RemoveAll(this);
		}
	}

	bool FHideObjectsNotInWorldLogic::ShouldShowObject(const FSoftObjectPath& ObjectPath) const
	{
		if (!GWorld)
		{
			return false;
		}

		const UObject* Object = ObjectPath.ResolveObject();
		if (!Object)
		{
			// If it does not resolve, it's no in GWorld (the world should load all the actors in it).
			return false;
		}

		if (Object->IsIn(GWorld))
		{
			return true;
		}

		const bool bIsInSublevel = GWorld->GetLevels().ContainsByPredicate([&Object](ULevel* Level)
		{
			return Level && Object->IsIn(Level);
		});
		return bIsInSublevel;
	}
}
