// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Editor/UnrealEditor/ModifyObjectInLevelHandler.h"

#include "Replication/Editor/Model/IEditableReplicationStreamModel.h"

#include "Engine/Engine.h"
#include "GameFramework/Actor.h"
#include "UObject/UObjectGlobals.h"

namespace UE::ConcertClientSharedSlate
{
	FModifyObjectInLevelHandler::FModifyObjectInLevelHandler(
		ConcertSharedSlate::IEditableReplicationStreamModel& UpdatedModel
		)
		: UpdatedModel(UpdatedModel)
	{
		if (ensureMsgf(GEngine, TEXT("Expected to be valid at this stage")))
		{
			GEngine->OnLevelActorDeleted().AddRaw(this, &FModifyObjectInLevelHandler::OnActorDeleted);
		}

		// This handles a lot of cases where subobjects are added / remove to the actor hierarchy
		FCoreUObjectDelegates::OnObjectTransacted.AddRaw(this, &FModifyObjectInLevelHandler::OnObjectTransacted);
	}

	FModifyObjectInLevelHandler::~FModifyObjectInLevelHandler()
	{
		if (GEngine)
		{
			GEngine->OnLevelActorDeleted().RemoveAll(this);
		}
		FCoreUObjectDelegates::OnObjectTransacted.RemoveAll(this);
	}

	void FModifyObjectInLevelHandler::OnActorDeleted(AActor* Actor) const
	{
		const FSoftObjectPath ActorPath { Actor };
		TArray<FSoftObjectPath> RemovedObjects { ActorPath };
		UpdatedModel.ForEachSubobject(ActorPath, [&RemovedObjects](const FSoftObjectPath& Subobject)
		{
			RemovedObjects.Add(Subobject); return EBreakBehavior::Continue;
		});
		
		if (!RemovedObjects.IsEmpty())
		{
			UpdatedModel.RemoveObjects(RemovedObjects);
		}
	}

	void FModifyObjectInLevelHandler::OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent) const
	{
		const bool bIsContainedInModel = UpdatedModel.ContainsObjects({ Object });
		bool bContainsAnyChildInModel = false;
		UpdatedModel.ForEachSubobject(Object, [&bContainsAnyChildInModel](const FSoftObjectPath& Child)
		{
			bContainsAnyChildInModel = true;
			return EBreakBehavior::Break;
		});

		if (bIsContainedInModel || bContainsAnyChildInModel)
		{
			OnHierarchyNeedsRefreshDelegate.Broadcast();
		}
	}
}
