// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Set.h"
#include "Misc/IFilter.h"

class FFieldClass;
namespace UE::ConcertSharedSlate { class FPropertyData; }

namespace UE::ConcertClientSharedSlate
{
	/**
	 * Base class for property filters.
	 * 
	 * This filter inverts the results: all filters are supposed to be inverse filters, i.e. they should be run when
	 * they are greyed out in the UI. When they are run, they should remove specific properties.
	 *
	 * Subclasses simply implement Matches, which figures out whether the property is contained in some FPropertyData.
	 */
	class FPropertyFilterBase : public IFilter<const ConcertSharedSlate::FPropertyData&>
	{
	public:
		
		//~ Begin IFilter Interface
		virtual bool PassesFilter(const ConcertSharedSlate::FPropertyData& InItem) const final override
		{
			return MatchesFilteredForProperty(InItem);
		}
		virtual FChangedEvent& OnChanged() final override { return ChangedEventDelegate; }
		//~ End IFilter Interface

	private:

		/** Dummy delegate to fulfill IFilter interface: this filter will never change so this delegate will never trigger. */
		FChangedEvent ChangedEventDelegate;

		/** @return Whether this item contains the property this filter is looking for.*/
		virtual bool MatchesFilteredForProperty(const ConcertSharedSlate::FPropertyData& InItem) const = 0;
	};
}

