// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyFilter_ByPropertyType.h"

#include "Replication/PropertyChainUtils.h"
#include "Replication/Editor/Model/Data/PropertyData.h"

#include "UObject/UnrealType.h"

namespace UE::ConcertClientSharedSlate
{
	bool FPropertyFilter_ByPropertyType::MatchesFilteredForProperty(const ConcertSharedSlate::FPropertyData& InItem) const
	{
		UClass* Class = InItem.GetOwningClassPtr().LoadSynchronous();
		if (!Class)
		{
			return false;
		}

		const FProperty* Property = ConcertSyncCore::PropertyChain::ResolveProperty(*Class, InItem.GetProperty());
		return Property && AllowedClasses.Contains(Property->GetClass());
	}
}
