// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyFilterBase.h"
#include "Containers/Set.h"
#include "Misc/IFilter.h"

class FFieldClass;

namespace UE::ConcertSharedSlate { class FPropertyData; }

namespace UE::ConcertClientSharedSlate
{
	/** Filters a property based on the FFieldClass it has. */
	class FPropertyFilter_ByPropertyType : public FPropertyFilterBase
	{
	public:

		FPropertyFilter_ByPropertyType(TSet<FFieldClass*> AllowedClasses)
			: AllowedClasses(MoveTemp(AllowedClasses))
		{}

	private:

		TSet<FFieldClass*> AllowedClasses;

		/** Dummy delegate to fulfill IFilter interface: this filter will never change so this delegate will never trigger. */
		FChangedEvent ChangedEventDelegate;

		//~ Begin FPropertyFilterBase Interface
		virtual bool MatchesFilteredForProperty(const ConcertSharedSlate::FPropertyData& InItem) const override;
		//~ End FPropertyFilterBase Interface
	};
}

