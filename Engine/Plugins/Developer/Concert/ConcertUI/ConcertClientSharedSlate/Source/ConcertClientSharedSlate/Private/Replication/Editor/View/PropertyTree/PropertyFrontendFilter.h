// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/FilterBase.h"
#include "Internationalization/Text.h"

namespace UE::ConcertSharedSlate { class FPropertyData; }

namespace UE::ConcertClientSharedSlate
{
	/** Base UI filter for filtering properties. Gets rid of all the extra functionality in FFilterBase we do not need. */
	class FPropertyFrontendFilter : public FFilterBase<const ConcertSharedSlate::FPropertyData&>
	{
	public:

		FPropertyFrontendFilter(TSharedRef<FFilterCategory> InCategory, FText DisplayName, FText Tooltip = FText::GetEmpty())
			: FFilterBase(MoveTemp(InCategory))
			, DisplayName(MoveTemp(DisplayName))
			, Tooltip(MoveTemp(Tooltip))
		{}

		//~ Begin FFilterBase Interface
		virtual FString GetName() const override { return DisplayName.ToString(); }
		virtual FText GetDisplayName() const override { return DisplayName; }
		virtual FText GetToolTipText() const override { return Tooltip; }
		virtual FLinearColor GetColor() const override { return FLinearColor(0.6f, 0.6f, 0.6f, 0.6f); }
		virtual FName GetIconName() const override { return NAME_None; }
		virtual bool IsInverseFilter() const override { return false; }
		virtual void ActiveStateChanged(bool bActive) override {}
		virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override {}
		virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override {}
		virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override {}
		//~ End FFilterBase Interface

	private:

		FText DisplayName;
		FText Tooltip;
	};

	/** Inlines a IFilter<const FPropertyData&> at compile time to avoid an additional TSharedPtr<IFilter<const FPropertyData&>>. */
	template<typename TFilterType>
	class TPropertyFrontendFilter : public FPropertyFrontendFilter
	{
	public:

		template<typename... TArg>
		TPropertyFrontendFilter(TSharedRef<FFilterCategory> InCategory, FText DisplayName, TArg&&... Arg)
			: FPropertyFrontendFilter(MoveTemp(InCategory), MoveTemp(DisplayName))
			, FilterImplementation(Forward<TArg>(Arg)...)
		{
			FilterImplementation.OnChanged().AddLambda([this]()
			{
				FPropertyFrontendFilter::BroadcastChangedEvent();
			});
		}

		template<typename... TArg>
		TPropertyFrontendFilter(TSharedRef<FFilterCategory> InCategory, FText DisplayName, FText Tooltip, TArg&&... Arg)
			: FPropertyFrontendFilter(MoveTemp(InCategory), MoveTemp(DisplayName), MoveTemp(Tooltip))
			, FilterImplementation(Forward<TArg>(Arg)...)
		{
			FilterImplementation.OnChanged().AddLambda([this]()
			{
				FPropertyFrontendFilter::BroadcastChangedEvent();
			});
		}
		
		virtual bool PassesFilter(const ConcertSharedSlate::FPropertyData& InItem) const override
		{
			return FilterImplementation.PassesFilter(InItem);
		}

	private:
		
		TFilterType FilterImplementation;
	};
}

