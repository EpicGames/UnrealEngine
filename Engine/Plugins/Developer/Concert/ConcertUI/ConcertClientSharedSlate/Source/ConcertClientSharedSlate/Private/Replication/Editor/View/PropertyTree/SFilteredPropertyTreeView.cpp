// Copyright Epic Games, Inc. All Rights Reserved.

#include "SFilteredPropertyTreeView.h"

#include "Filters/SBasicFilterBar.h"
#include "PropertyFilter_ByPropertyType.h"
#include "PropertyFrontendFilter.h"
#include "Replication/ClientReplicationWidgetFactories.h"
#include "Replication/ReplicationWidgetFactories.h"
#include "Replication/Editor/Model/Data/PropertyData.h"

#include "UObject/UnrealType.h"

#define LOCTEXT_NAMESPACE "SReplicatedPropertiesView"

namespace UE::ConcertClientSharedSlate
{
	/** Exposes SetFrontendFilterActive so we can manually enable the default filters */
	class SReplicationFilterBar : public SBasicFilterBar<const ConcertSharedSlate::FPropertyData&>
	{
		using Super = SBasicFilterBar<const ConcertSharedSlate::FPropertyData&>;
	public:

		SLATE_BEGIN_ARGS(SReplicationFilterBar)
		{}
			SLATE_EVENT(FOnFilterChanged, OnFilterChanged)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, TArray<SFilteredPropertyTreeView::FFilterRef> AllFilters)
		{
			Super::Construct(
			Super::FArguments()
				.FilterPillStyle(EFilterPillStyle::Basic)
				.CustomFilters(MoveTemp(AllFilters))
				.OnFilterChanged(InArgs._OnFilterChanged)
				.UseSectionsForCategories(true)
				);
		}

		// Expose from SBasicFilterBar
		using Super::SetFrontendFilterActive;

		void SetFilterVisuallyEnabled(const SFilteredPropertyTreeView::FFilterRef& Filter, bool bEnabled)
		{
			const TSharedRef<SFilter>* FilterWidget = Filters.FindByPredicate([&Filter](const TSharedRef<SFilter>& FilterWidget)
			{
				return FilterWidget->GetFrontendFilter() == Filter;
			});
			if (ensure(FilterWidget))
			{
				FilterWidget->Get().SetEnabled(bEnabled);
			}
		}
	};
	
	void SFilteredPropertyTreeView::Construct(const FArguments& InArgs, FFilterablePropertyTreeViewParams Params)
	{
		const FBuildFilterBarResult Filters = BuildFilterBar();

		using namespace UE::ConcertSharedSlate;
		FCreatePropertyTreeViewParams TreeViewParams
		{
			.PropertyColumns = MoveTemp(Params.AdditionalPropertyColumns),
			.FilterItem = FFilterPropertyData::CreateSP(this, &SFilteredPropertyTreeView::PassesFilters),
			.CreateCategoryRow = MoveTemp(Params.CreateCategoryRow),
			.PrimaryPropertySort = Params.PrimaryPropertySort,
			.SecondaryPropertySort = Params.SecondaryPropertySort
		};
		TreeViewParams.LeftOfPropertySearchBar.Widget = SBasicFilterBar<const FPropertyData&>::MakeAddFilterButton(FilterBar.ToSharedRef());
		TreeViewParams.RowBelowSearchBar.Widget = FilterBar.ToSharedRef();
		TreeViewParams.NoItemsContent.Widget = SNew(STextBlock).Text(LOCTEXT("AllFitlered", "All properties filtered."));
			
		ExtendedTreeView = CreateSearchablePropertyTreeView(MoveTemp(TreeViewParams));
		
		ChildSlot
		[
			ExtendedTreeView->GetWidget()
		];

		// For better UX, hide certain properties by default (e.g. why would you want to replicate bools?)
		// Do this AFTER initializing ReplicatedProperties because it triggers the OnItemsChanged callback.
		for (const FFilterRef& Filter : Filters.DisabledByDefault)
		{
			FilterBar->SetFilterCheckState(Filter, ECheckBoxState::Unchecked);
		}
		// Show all the other filters as enabled (not greyed out: blue) - they will not run their logic since they are inverse.
		for (const FFilterRef& Filter : Filters.EnabledByDefault)
		{
			// Makes it appear on the bar
			FilterBar->SetFilterCheckState(Filter, ECheckBoxState::Checked);
			// Functionally makes it affect the search
			FilterBar->SetFrontendFilterActive(Filter, true);
			// Visually makes the button blue (i.e. so it looks enabled)
			FilterBar->SetFilterVisuallyEnabled(Filter, true);
		}
	}

	SFilteredPropertyTreeView::FBuildFilterBarResult SFilteredPropertyTreeView::BuildFilterBar()
	{
		TSharedRef<FFilterCategory> CommonCategory = MakeShared<FFilterCategory>(
			LOCTEXT("CommonCategory.Name", "Common"),
			LOCTEXT("CommonCategory.ToolTip", "Include commonly replicated properties.")
		);
		TSharedRef<FFilterCategory> UncommonCategory = MakeShared<FFilterCategory>(
			LOCTEXT("UncommonCategory.Advanced", "Uncommon"),
			LOCTEXT("UncommonCategory.ToolTip", "Include uncommonly replicated properties.")
		);

		FBuildFilterBarResult Result;
		using FFrontendFilter = TPropertyFrontendFilter<FPropertyFilter_ByPropertyType>;
		// Do not show under search bar
		Result.DisabledByDefault =
		{
			// Ordering matters for the drop-down menu next to settings
			// Common
			MakeShared<FFrontendFilter>(CommonCategory, LOCTEXT("Ints", "Integer"), LOCTEXT("Ints.Tooltip", "Includes: uint16, uint32, uint64, int16, int32, int64"), TSet<FFieldClass*>{ FUInt16Property::StaticClass(), FUInt32Property::StaticClass(), FUInt64Property::StaticClass(), FInt16Property::StaticClass(), FIntProperty::StaticClass(), FInt64Property::StaticClass() }),
			MakeShared<FFrontendFilter>(CommonCategory, LOCTEXT("Floats", "Float"), LOCTEXT("Floats.Tooltip", "Includes: float, double"), TSet<FFieldClass*>{ FFloatProperty::StaticClass(), FDoubleProperty::StaticClass() }),
			MakeShared<FFrontendFilter>(CommonCategory, LOCTEXT("Struct", "Struct"), TSet<FFieldClass*>{ FStructProperty::StaticClass() }),
			MakeShared<FFrontendFilter>(CommonCategory, LOCTEXT("Containers", "Containers"), LOCTEXT("Containers.Tooltip", "Includes: array, set, map"), TSet<FFieldClass*>{ FArrayProperty::StaticClass(), FSetProperty::StaticClass(), FMapProperty::StaticClass() }),

			// Uncommon
			MakeShared<FFrontendFilter>(UncommonCategory, LOCTEXT("Bool", "Boolean"), TSet<FFieldClass*>{ FBoolProperty::StaticClass() }),
			MakeShared<FFrontendFilter>(UncommonCategory, LOCTEXT("Enum", "Enum"), TSet<FFieldClass*>{ FEnumProperty::StaticClass(), FByteProperty::StaticClass() }),
			MakeShared<FFrontendFilter>(UncommonCategory, LOCTEXT("Text", "Text"), LOCTEXT("Text.Tooltip", "Includes: FName, FString, FText"), TSet<FFieldClass*>{ FNameProperty::StaticClass(), FStrProperty::StaticClass(), FTextProperty::StaticClass() }),
			MakeShared<FFrontendFilter>(UncommonCategory, LOCTEXT("SoftPtr", "Soft Ptr"), TSet<FFieldClass*>{ FSoftObjectProperty::StaticClass() }),
		};
		// Show up under search bar as enabled
		Result.EnabledByDefault =
		{
			// We'll not enable any filters by default because that's the default for other places in the engine, like the Content Browser, too
		};
		// Show up in menu
		TArray<FFilterRef> AllFilters;
		AllFilters.Append(Result.EnabledByDefault);
		AllFilters.Append(Result.DisabledByDefault);
		
		FilterBar = SNew(SReplicationFilterBar, MoveTemp(AllFilters))
			.OnFilterChanged(this, &SFilteredPropertyTreeView::RequestRefilter);
		
		return Result;
	}

	ConcertSharedSlate::EFilterResult SFilteredPropertyTreeView::PassesFilters(const ConcertSharedSlate::FPropertyData& ReplicatedPropertyData) const
	{
		const bool bPassesFilter = FilterBar->GetAllActiveFilters()->Num() == 0 // Return all items when none enabled
			|| PassesAnyFilters(ReplicatedPropertyData);
		return bPassesFilter ? ConcertSharedSlate::EFilterResult::PassesFilter : ConcertSharedSlate::EFilterResult::DoesNotPassFilter;
	}

	bool SFilteredPropertyTreeView::PassesAnyFilters(const ConcertSharedSlate::FPropertyData& ReplicatedPropertyData) const
	{
		TSharedPtr<TFilterCollection<const ConcertSharedSlate::FPropertyData&>> FilterCollection = FilterBar->GetAllActiveFilters();
		for (const TSharedPtr<IFilter<const ConcertSharedSlate::FPropertyData&>>& Filter : *FilterCollection.Get())
		{
			if (Filter->PassesFilter(ReplicatedPropertyData))
			{
				return true;
			}
		}
		return false;
	}
}

#undef LOCTEXT_NAMESPACE