// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/View/IPropertyTreeView.h"
#include "Replication/Editor/View/Column/ReplicationColumnsUtils.h"

#include "Filters/SBasicFilterBar.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertSharedSlate
{
	class FPropertyData;
	enum class EFilterResult : uint8;
}

namespace UE::ConcertClientSharedSlate
{
	class SReplicationFilterBar;
	struct FFilterablePropertyTreeViewParams;
	
	/** This widget extends SPropertyTreeView with filtering functionality. */
	class SFilteredPropertyTreeView
		: public SCompoundWidget
		, public ConcertSharedSlate::IPropertyTreeView
	{
	public:
		
		using FFilterRef = TSharedRef<FFilterBase<const ConcertSharedSlate::FPropertyData&>>;
		
		SLATE_BEGIN_ARGS(SFilteredPropertyTreeView)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FFilterablePropertyTreeViewParams Params);
		
		//~ Begin IPropertyTreeView Interface
		virtual void RefreshPropertyData(const TArray<ConcertSharedSlate::FPropertyAssignmentEntry>& Entries, bool bCanReuseExistingRowItems) override
		{
			ExtendedTreeView->RefreshPropertyData(Entries, bCanReuseExistingRowItems);
		}
		virtual void RequestRefilter() const override { ExtendedTreeView->RequestRefilter(); }
		virtual void RequestResortForColumn(const FName& ColumnId) override { ExtendedTreeView->RequestResortForColumn(ColumnId); }
		virtual void RequestScrollIntoView(const FConcertPropertyChain& PropertyChain) override { ExtendedTreeView->RequestScrollIntoView(PropertyChain); }
		virtual TSharedRef<SWidget> GetWidget() override { return SharedThis(this); }
		//~ End IPropertyTreeView Interface

	private:

		/** The tree view displaying the replicated properties */
		TSharedPtr<IPropertyTreeView> ExtendedTreeView;
		/** Displays the active filters*/
		TSharedPtr<SReplicationFilterBar> FilterBar;

		struct FBuildFilterBarResult
		{
			TArray<FFilterRef> EnabledByDefault;
			TArray<FFilterRef> DisabledByDefault;
		};
		/** Build the filter bar and returns the filters that should be active by default. */
		FBuildFilterBarResult BuildFilterBar();

		/** Runs all filters through this item */
		ConcertSharedSlate::EFilterResult PassesFilters(const ConcertSharedSlate::FPropertyData& ReplicatedPropertyData) const;
		bool PassesAnyFilters(const ConcertSharedSlate::FPropertyData& ReplicatedPropertyData) const;
	};
}

