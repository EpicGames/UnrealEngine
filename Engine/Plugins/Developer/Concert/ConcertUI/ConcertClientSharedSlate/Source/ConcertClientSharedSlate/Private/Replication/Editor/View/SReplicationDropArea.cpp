// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReplicationDropArea.h"

#include "Misc/EBreakBehavior.h"

#include "EditorActorFolders.h"
#include "SDropTarget.h"
#include "DragAndDrop/ActorDragDropOp.h"
#include "DragAndDrop/CompositeDragDropOp.h"
#include "DragAndDrop/FolderDragDropOp.h"

#include <type_traits>

namespace UE::ConcertClientSharedSlate
{
	template<typename TCallback> requires std::is_invocable_r_v<EBreakBehavior, TCallback, UObject*>
	static void ForEachObjectInOperation(const TSharedPtr<FDragDropOperation>& InOperation, TCallback&& Callback)
	{
		if (!InOperation || !InOperation.IsValid())
		{
			return;
		}
		
		TSharedPtr<FActorDragDropOp>  ActorDrag = nullptr;
		TSharedPtr<FFolderDragDropOp> FolderDrag = nullptr;

		if (InOperation->IsOfType<FActorDragDropOp>())
		{
			ActorDrag = StaticCastSharedPtr<FActorDragDropOp>(InOperation);
		}
		else if (InOperation->IsOfType<FFolderDragDropOp>())
		{
			FolderDrag = StaticCastSharedPtr<FFolderDragDropOp>(InOperation);
		}
		else if (InOperation->IsOfType<FCompositeDragDropOp>())
		{
			if (const TSharedPtr<FCompositeDragDropOp> CompositeDrag = StaticCastSharedPtr<FCompositeDragDropOp>(InOperation))
			{
				ActorDrag = CompositeDrag->GetSubOp<FActorDragDropOp>();
				FolderDrag = CompositeDrag->GetSubOp<FFolderDragDropOp>();
			}
		}

		if (ActorDrag)
		{
			for (TWeakObjectPtr<AActor> WeakActor : ActorDrag->Actors)
			{
				if (AActor* Actor = WeakActor.Get())
				{
					if (Callback(Actor) == EBreakBehavior::Break)
					{
						return;
					}
				}
			}
		}

		if (FolderDrag)
		{
			TArray<AActor*> FolderActors;
			FActorFolders::GetActorsFromFolders(*GWorld, FolderDrag->Folders, FolderActors);

			for (AActor* ActorInFolder : FolderActors)
			{
				if (Callback(ActorInFolder) == EBreakBehavior::Break)
				{
					return;
				}
			}
		}
	}
}

namespace UE::ConcertClientSharedSlate
{
	void SReplicationDropArea::Construct(const FArguments& InArgs)
	{
		HandleDroppedObjectsDelegate = InArgs._HandleDroppedObjects;
		CanDropObjectDelegate = InArgs._CanDropObject;
		
		ChildSlot
		[
			SNew(SDropTarget)
			.OnDropped(this, &SReplicationDropArea::OnDragDropTarget)
			.OnAllowDrop(this, &SReplicationDropArea::CanDragDropTarget)
			.OnIsRecognized(this, &SReplicationDropArea::CanDragDropTarget)
			[
				InArgs._Content.Widget
			]
		];
	}

	FReply SReplicationDropArea::OnDragDropTarget(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent) const
	{
		TArray<UObject*> AllowedDroppedObjects;
		ForEachObjectInOperation(DragDropEvent.GetOperation(), [this, &AllowedDroppedObjects](UObject* Object)
		{
			if (CanDrop(Object))
			{
				AllowedDroppedObjects.Add(Object);
			}
			return EBreakBehavior::Continue;
		});

		if (ensure(!AllowedDroppedObjects.IsEmpty()))
		{
			HandleDroppedObjectsDelegate.Execute(AllowedDroppedObjects);
		}
		
		return FReply::Handled();
	}

	bool SReplicationDropArea::CanDragDropTarget(TSharedPtr<FDragDropOperation> InOperation) const
	{
		bool bCanHandle = false;
		ForEachObjectInOperation(InOperation, [this, &bCanHandle](UObject* Object)
		{
			bCanHandle = CanDrop(Object);
			return bCanHandle ? EBreakBehavior::Break : EBreakBehavior::Continue;
		});
		return bCanHandle;
	}

	bool SReplicationDropArea::CanDrop(UObject* Object) const
	{
		return Object && (!CanDropObjectDelegate.IsBound() || CanDropObjectDelegate.Execute(*Object));
	}
}
