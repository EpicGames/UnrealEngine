// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/ClientReplicationWidgetDelegates.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertClientSharedSlate
{
	/** Handles drag-drop operations for the replication UI. */
	class SReplicationDropArea : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SReplicationDropArea) {}
			/** Required. Responds to objects being dropped into the replication outliner. */
			SLATE_EVENT(FDragDropReplicatableObject, HandleDroppedObjects)
			/** Optional. Decides whether a dragged object can be dropped. */
			SLATE_EVENT(FCanDragDropObject, CanDropObject)
			
			SLATE_DEFAULT_SLOT(FArguments, Content)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:

		/** Responds to objects being dropped into the replication outliner. */
		FDragDropReplicatableObject HandleDroppedObjectsDelegate;
		/** Decides whether a dragged object can be dropped. */
		FCanDragDropObject CanDropObjectDelegate;

		FReply OnDragDropTarget(const FGeometry& Geometry, const FDragDropEvent& DragDropEvent) const;
		bool CanDragDropTarget(TSharedPtr<FDragDropOperation> InOperation) const;

		/** @return Whether Object can be dropped. */
		bool CanDrop(UObject* Object) const;
	};
}

