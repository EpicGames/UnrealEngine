// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/ObjectNameUtils.h"

#include "Internationalization/Text.h"
#include "Misc/ObjectUtils.h"

#include "GameFramework/Actor.h"
#include "SubobjectDataSubsystem.h"

namespace UE::ConcertClientSharedSlate
{
	namespace Private
	{
		static FText FindSubobjectDisplayName(const UObject& Subbject, AActor& OwningActor)
		{
			USubobjectDataSubsystem* SubobjectDataSubsystem = USubobjectDataSubsystem::Get();
			TArray<FSubobjectDataHandle> Handles;
			SubobjectDataSubsystem->GatherSubobjectData(&OwningActor, Handles);
				
			for (const FSubobjectDataHandle& Handle : Handles)
			{
				const FSubobjectData* SubobjectData = Handle.GetData();
				const UObject* Object = SubobjectData->FindComponentInstanceInActor(&OwningActor);
				if (Object == &Subbject)
				{
					constexpr bool bShowNativeComponentNames = false;
					return FText::FromString(SubobjectData->GetDisplayString(bShowNativeComponentNames));
				}
			}

			return FText::GetEmpty();
		}
	}

	FText GetObjectDisplayName(const TSoftObjectPtr<>& ObjectPath)
	{
		if (UObject* ResolvedObject = ObjectPath.Get())
		{
			// Display actor just like the outliner does
			if (const AActor* AsActor = Cast<AActor>(ResolvedObject))
			{
				return FText::FromString(AsActor->GetActorLabel());
			}

			// Display the same component name as the SSubobjectEditor widget does, i.e. component hierarchy in the details panel or Blueprint editor.
			AActor* OwningActor = ResolvedObject->GetTypedOuter<AActor>();
			const FText FoundSubobjectName = OwningActor ? Private::FindSubobjectDisplayName(*ResolvedObject, *OwningActor) : FText::GetEmpty();
			if (!FoundSubobjectName.IsEmpty())
			{
				return FoundSubobjectName;
			}

			return FText::FromString(ResolvedObject->GetName());
		}
		
		return FText::FromString(ConcertSyncCore::ExtractObjectNameFromPath(ObjectPath.GetUniqueID()));
	}
}
