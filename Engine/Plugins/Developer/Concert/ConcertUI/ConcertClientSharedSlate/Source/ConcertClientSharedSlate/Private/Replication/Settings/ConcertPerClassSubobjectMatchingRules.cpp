// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Settings/ConcertPerClassSubobjectMatchingRules.h"

#include "UObject/Class.h"

namespace UE::ConcertSharedSlate::DefaultSubobjects
{
	static void InternalAddAdditionalObjectsFromSettings(UClass& StartClass, const FConcertPerClassSubobjectMatchingRules& Settings, const UObject& AddedObject, TFunctionRef<EBreakBehavior(UObject&)> OnSubobjectMatched)
	{
		// Find the most specialized class properties
		UClass* Current = &StartClass;
		const FConcertInheritableSubobjectMatchingRules* DefaultSubobjects = nullptr;
		for (; Current && !DefaultSubobjects; Current = Current->GetSuperClass())
		{
			DefaultSubobjects = Settings.SubobjectMatchingRules.Find(Current);
			if (!DefaultSubobjects)
			{
				continue;
			}

			EBreakBehavior BreakResult = EBreakBehavior::Continue;
			DefaultSubobjects->MatchToSubobjectsBreakable(AddedObject, [&OnSubobjectMatched, &BreakResult](UObject& Object)
			{
				BreakResult = OnSubobjectMatched(Object);
				return BreakResult;
			});
			if (BreakResult == EBreakBehavior::Break)
			{
				return;
			}
			
			// Recurse super structs
			if (UClass* Parent = Current->GetSuperClass()
				; Parent && DefaultSubobjects->bInheritFromBase)
			{
				InternalAddAdditionalObjectsFromSettings(*Parent, Settings, AddedObject, OnSubobjectMatched);
			}
		}
	}
}

void FConcertPerClassSubobjectMatchingRules::MatchSubobjectsRecursivelyBreakable(const UObject& Object, TFunctionRef<EBreakBehavior(UObject&)> OnSubobjectMatched) const
{
	checkf(!Object.IsA<UClass>(), TEXT("Pass in the UObject instanced directly, not its class!"));
	UE::ConcertSharedSlate::DefaultSubobjects::InternalAddAdditionalObjectsFromSettings(*Object.GetClass(), *this, Object, OnSubobjectMatched);
}