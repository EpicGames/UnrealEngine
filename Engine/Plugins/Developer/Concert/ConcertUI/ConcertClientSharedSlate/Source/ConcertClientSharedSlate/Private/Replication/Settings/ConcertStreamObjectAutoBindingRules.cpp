// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Settings/ConcertStreamObjectAutoBindingRules.h"

#include "ConcertLogGlobal.h"
#include "Replication/PropertyChainUtils.h"

#include "Algo/IndexOf.h"
#include "UObject/Class.h"

namespace UE::ConcertSharedSlate::DefaultProperties
{
	static void ApplyDefaultPropertySelection(TFunctionRef<void(FConcertPropertyChain&& Chain)> Callback, const FConcertDefaultPropertySelection& Selection, UStruct& Class)
	{
		// Preparse the the FStrings into paths
		TArray<TArray<FName>> Paths;
		Paths.Reserve(Selection.DefaultSelectedProperties.Num());
		for (const FString& StringPath : Selection.DefaultSelectedProperties)
		{
			TArray<FString> Parts;
			StringPath.ParseIntoArray(Parts, TEXT("."));
			TArray<FName> NameArray;
			Algo::Transform(Parts, NameArray, [](const FString& String){ return *String; });
			Paths.Emplace(MoveTemp(NameArray));
		}

		// This walks through the entire property hierarchy once
		using namespace ConcertSyncCore::PropertyChain;
		BulkConstructConcertChainsFromPaths(Class, Paths.Num(), [&Callback, &Paths](const FArchiveSerializedPropertyChain& Chain, const FProperty& LeafProperty)
		{
			const int32 IndexOfMatches = Algo::IndexOfByPredicate(Paths, [&Chain, &LeafProperty](const TArray<FName>& Path){ return DoPathAndChainsMatch(Path, Chain, LeafProperty); });
			const bool bMatches = IndexOfMatches != INDEX_NONE;
			if (bMatches)
			{
				Callback(FConcertPropertyChain(&Chain, LeafProperty));
				Paths.RemoveAtSwap(IndexOfMatches);
			}
			return bMatches;
		});

		// Warn the user of unmatched properties so they can de-clutter their settings of useless entries
		for (const TArray<FName>& Path : Paths)
		{
			UE_LOGF(LogConcert, Warning,
				"Path %ls was unmatched. Fix your settings for class %ls.",
				*FString::JoinBy(Path, TEXT("."), [](const FName& Name){ return Name.ToString(); }),
				*Class.GetName()
				);
		}
	}
}

void FConcertStreamObjectAutoBindingRules::AddDefaultPropertiesFromSettings(UClass& Class, TFunctionRef<void(FConcertPropertyChain&& Chain)> Callback) const
{
	// Find the most specialized class properties
	UClass* Current = &Class;
	const FConcertDefaultPropertySelection* DefaultProperties = nullptr;
	for (; Current && !DefaultProperties; Current = Current->GetSuperClass())
	{
		DefaultProperties = DefaultPropertySelection.Find(Current);
		if (!DefaultProperties)
		{
			continue;
		}
			
		UE::ConcertSharedSlate::DefaultProperties::ApplyDefaultPropertySelection(Callback, *DefaultProperties, *Current);
		// Recurse super structs
		if (UClass* Parent = Current->GetSuperClass()
			; Parent && DefaultProperties->bInheritFromBase)
		{
			AddDefaultPropertiesFromSettings(*Parent, Callback);
		}
	}
}