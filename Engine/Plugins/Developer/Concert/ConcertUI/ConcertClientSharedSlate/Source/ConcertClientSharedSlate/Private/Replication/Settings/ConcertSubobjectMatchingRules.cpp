// Copyright Epic Games, Inc. All Rights Reserved.

#include "Replication/Settings/ConcertSubobjectMatchingRules.h"

#include "Components/ActorComponent.h"
#include "Internationalization/Regex.h"
#include "Misc/EBreakBehavior.h"

#include "UObject/ObjectMacros.h"

namespace UE::ConcertSharedSlate
{
	static bool MatchesAnyRegex(const FString& Input, const TSet<FString>& AllRegex)
	{
		for (const FString& RegexString : AllRegex)
		{
			const FRegexPattern Pattern(RegexString);
			FRegexMatcher Matcher(Pattern, Input);
			if (Matcher.FindNext())
			{
				return true;
			}
		}

		return false;
	}
}

void FConcertSubobjectMatchingRules::MatchToSubobjectsBreakable(const UObject& AddedObject, TFunctionRef<EBreakBehavior(UObject&)> OnSubobjectMatched) const
{
	// Do not search recursively so FurtherObjectsCallback can decide to call AddAdditionalObjectsFromSettings again on the newly added objects.
	ForEachObjectWithOuterBreakable(&AddedObject, [this, &OnSubobjectMatched](UObject* Subobject)
	{
		// Has exclusion regex?
		const bool bShouldExclude = UE::ConcertSharedSlate::MatchesAnyRegex(Subobject->GetName(), ExcludeSubobjectRegex);
		if (bShouldExclude)
		{
			return true;
		}

		// Was told to add all UActorComponents?
		const bool bIncludeAllSubobjects = IncludeAllOption == EConcertIncludeAllSubobjectsType::AllSubobjects;
		const bool bIncludeDueToComponent = Subobject->IsA(UActorComponent::StaticClass())
			&& IncludeAllOption == EConcertIncludeAllSubobjectsType::AllComponents;
		if (bIncludeAllSubobjects || bIncludeDueToComponent)
		{
			return OnSubobjectMatched(*Subobject) == EBreakBehavior::Break;
		}

		// Has configured class?
		UClass* CurrentClass = Subobject->GetClass();
		for (; CurrentClass; CurrentClass = CurrentClass->GetSuperClass())
		{
			if (IncludeClasses.Contains(Subobject->GetClass()))
			{
				return OnSubobjectMatched(*Subobject) == EBreakBehavior::Break;
			}
		}

		// Has inclusion regex?
		const bool bShouldIncludeByRegex = UE::ConcertSharedSlate::MatchesAnyRegex(Subobject->GetName(), IncludeSubobjectRegex);
		if (bShouldIncludeByRegex)
		{
			return OnSubobjectMatched(*Subobject) == EBreakBehavior::Break;
		}
		
		return true;
	}, EGetObjectsFlags::None, RF_MirroredGarbage);
}