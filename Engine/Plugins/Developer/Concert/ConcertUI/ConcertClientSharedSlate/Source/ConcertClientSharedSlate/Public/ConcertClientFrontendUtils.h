// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ConcertActionDefinition.h"
#include "Styling/AppStyle.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/SBoxPanel.h"
#include "ConcertFrontendStyle.h"

#define LOCTEXT_NAMESPACE "ConcertClientFrontendUtils"

namespace ConcertClientFrontendUtils
{
	static const FName ButtonIconSyle = TEXT("FontAwesome.10");
	static const float MinDesiredWidthForBtnAndIcon = 29.f;
	static const FName ButtonStyleNames[(int32)EConcertActionType::NUM] = {
		TEXT("SimpleButton"),
		TEXT("FlatButton.Primary"),
		TEXT("FlatButton.Info"),
		TEXT("FlatButton.Success"),
		TEXT("FlatButton.Warning"),
		TEXT("FlatButton.Danger"),
	};
	
	inline TSharedRef<SButton> CreateTextButton(const FConcertActionDefinition& InDef)
	{
		const FButtonStyle* ButtonStyle = &FAppStyle::Get().GetWidgetStyle<FButtonStyle>(ButtonStyleNames[(int32)InDef.Type]);
		check(ButtonStyle);
		const float ButtonContentWidthPadding = 6.f;
		const float PaddingCompensation = (ButtonStyle->NormalPadding.Left + ButtonStyle->NormalPadding.Right + ButtonContentWidthPadding * 2);

		return SNew(SButton)
			.ButtonStyle(ButtonStyle)
			.ToolTipText(InDef.ToolTipText)
			.ContentPadding(FMargin(ButtonContentWidthPadding, 2.f))
			.IsEnabled(InDef.IsEnabled)
			.Visibility_Lambda([IsVisible = InDef.IsVisible]() { return IsVisible.Get() ? EVisibility::Visible : EVisibility::Collapsed; })
			.OnClicked_Lambda([OnExecute = InDef.OnExecute]() { OnExecute.ExecuteIfBound(); return FReply::Handled(); })
			[
				SNew(SBox)
				.MinDesiredWidth(MinDesiredWidthForBtnAndIcon - PaddingCompensation)
				.HAlign(HAlign_Center)
				[
					SNew(STextBlock)
					.Font(FAppStyle::Get().GetFontStyle(ButtonIconSyle))
					.Text(InDef.Text)
					.Justification(ETextJustify::Center)
				]
			];
	}

	inline TSharedRef<SButton> CreateIconButton(const FConcertActionDefinition& InDef)
	{
		return SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), ButtonStyleNames[(int32)InDef.Type])
			.ToolTipText(InDef.ToolTipText)
			.ContentPadding(FMargin(0, 0))
			.IsEnabled(InDef.IsEnabled)
			.Visibility_Lambda([IsVisible = InDef.IsVisible]() { return IsVisible.Get() ? EVisibility::Visible : EVisibility::Collapsed; })
			.OnClicked_Lambda([OnExecute = InDef.OnExecute]() { OnExecute.ExecuteIfBound(); return FReply::Handled(); })
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			[
				SNew(SImage)
				.Image(TAttribute<const FSlateBrush*>::Create([IconStyleAttr = InDef.IconStyle]() { return FConcertFrontendStyle::Get()->GetBrush(IconStyleAttr.Get()); }))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
	}

	inline void AppendButtons(TSharedRef<SHorizontalBox> InHorizBox, TArrayView<const FConcertActionDefinition> InDefs)
	{
		for (const FConcertActionDefinition& Def : InDefs)
		{
			InHorizBox->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(FMargin(1.0f))
				[
					Def.IconStyle.IsSet() ? CreateIconButton(Def) : CreateTextButton(Def)
				];
		}
	}
};

#undef LOCTEXT_NAMESPACE
