// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Delegates/Delegate.h"

class UObject;

namespace UE::ConcertClientSharedSlate
{
	/** Responds to objects being dropped into the replication outliner. */
	DECLARE_DELEGATE_OneParam(FDragDropReplicatableObject, TConstArrayView<UObject*> DroppedObjects);
	/** Decides whether a dragged object can be dropped. */
	DECLARE_DELEGATE_RetVal_OneParam(bool, FCanDragDropObject, UObject& Object);
}