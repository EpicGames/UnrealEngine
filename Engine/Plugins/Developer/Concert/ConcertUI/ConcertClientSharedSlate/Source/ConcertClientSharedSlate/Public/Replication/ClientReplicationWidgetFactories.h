// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClientReplicationWidgetDelegates.h"
#include "Replication/PropertyTreeFactory.h"
#include "Templates/SharedPointer.h"

struct FConcertStreamObjectAutoBindingRules;
struct FConcertObjectReplicationMap;

namespace UE::ConcertSharedSlate
{
	class IPropertySourceProcessor;
	class IPropertyTreeView;
	class IEditableReplicationStreamModel;
	class IObjectNameModel;
	class IReplicationStreamEditor;
	class IStreamExtender;
	class IObjectHierarchyModel;
	
	struct FCreateEditorParams;
	struct FCreatePropertyTreeViewParams;
}

namespace UE::ConcertClientSharedSlate
{
	/** Builds a similar tree hierarchy as SSubobjectEditor. Reports only components as subobjects. */
	CONCERTCLIENTSHAREDSLATE_API TSharedRef<ConcertSharedSlate::IObjectHierarchyModel> CreateObjectHierarchyForComponentHierarchy();

	/** Name model that uses editor data for determining display names: actors use their labels, components ask USubobjectDataSubsystem. */
	CONCERTCLIENTSHAREDSLATE_API TSharedRef<ConcertSharedSlate::IObjectNameModel> CreateEditorObjectNameModel();
	
	/**
	 * Wraps the passed in BaseModel and makes it transactional.
	 * All calls that modify the underlying model was wrapped with scoped transactions.
	 * 
	 * @param OwnerObject The object containing the FConcertObjectReplicationMap - used for transactions.
	 */
	CONCERTCLIENTSHAREDSLATE_API TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> CreateTransactionalStreamModel(
		const TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel>& BaseModel,
		UObject& OwnerObject
		);
	/** Simpler CreateTransactionalStreamModel overload that internally creates an UObject and sets it up automatically. */
	CONCERTCLIENTSHAREDSLATE_API TSharedRef<ConcertSharedSlate::IEditableReplicationStreamModel> CreateTransactionalStreamModel();

	/** Params for creating a filterable property tree view. */
	struct FFilterablePropertyTreeViewParams
	{
		/** The columns the property view should have. The label column is always included. */
		TArray<ConcertSharedSlate::FPropertyColumnEntry> AdditionalPropertyColumns
		{
			ConcertSharedSlate::ReplicationColumns::Property::LabelColumn()
		};
		
		/** Optional initial primary sort mode for object rows */
		ConcertSharedSlate::FColumnSortInfo PrimaryPropertySort { ConcertSharedSlate::ReplicationColumns::Property::LabelColumnId, EColumnSortMode::Ascending };
		/** Optional initial secondary sort mode for object rows */
		ConcertSharedSlate::FColumnSortInfo SecondaryPropertySort { ConcertSharedSlate::ReplicationColumns::Property::LabelColumnId, EColumnSortMode::Ascending };
		
		/**
		 * Optional delegate for grouping objects under a category.
		 * If unset, no category are generated.
		 * 
		 * When the user clicks an object in the top view, this delegate will be called for the clicked object, its components (if an actor), and its (nested) subobjects.
		 * ContextObjects is a single object if a single object is clicked or multiple object in the case of multi-edit.
		 */
		ConcertSharedSlate::FCreateCategoryRow CreateCategoryRow;
	};
	
	/**
	 * Creates a tree view that allows filtering of properties based on their type.
	 * 
	 * There is a combo box to the left of the search bar for managing the used filters.
	 * The user can toggle used filters on and off under the search bar.
	 */
	CONCERTCLIENTSHAREDSLATE_API TSharedRef<ConcertSharedSlate::IPropertyTreeView> CreateFilterablePropertyTreeView(FFilterablePropertyTreeViewParams Params);

	struct FCreateDropTargetOutlinerWrapperParams
	{
		/** Required. Handles objects that were dropped. */
		FDragDropReplicatableObject HandleDroppedObjectsDelegate;
		/** Optional. Decides whether a dragged object can be dropped. */
		FCanDragDropObject CanDropObjectDelegate;
	};
	
	/** Returns a delegate that wraps the replication outliner with a SDropTarget: it allows users to drag-drop actors. */
	CONCERTCLIENTSHAREDSLATE_API ConcertSharedSlate::FWrapOutlinerWidget CreateDropTargetOutlinerWrapper(
		FCreateDropTargetOutlinerWrapperParams Params
		);
}
