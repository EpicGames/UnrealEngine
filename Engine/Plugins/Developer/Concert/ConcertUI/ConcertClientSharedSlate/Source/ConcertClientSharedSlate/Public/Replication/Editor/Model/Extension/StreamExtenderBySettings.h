// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/Extension/IStreamExtender.h"
#include "Misc/Attribute.h"

class UObject;
struct FConcertStreamObjectAutoBindingRules;

namespace UE::ConcertClientSharedSlate
{
	/** Automatically adds objects and properties according to settings specified in some FConcertReplicationEditorSettings. */
	class CONCERTCLIENTSHAREDSLATE_API FStreamExtenderBySettings : public ConcertSharedSlate::IStreamExtender
	{
	public:
		
		FStreamExtenderBySettings(TAttribute<const FConcertStreamObjectAutoBindingRules*> InReplicationSettingsAttribute);

		//~ Begin IStreamExtender Interface
		virtual void ExtendStream(UObject& ExtendedObject, ConcertSharedSlate::IStreamExtensionContext& Context) override;
		//~ End IStreamExtender Interface

	private:
		
		/** Optional settings for auto adding common properties and objects. */
		TAttribute<const FConcertStreamObjectAutoBindingRules*> ReplicationSettingsAttribute;
	};
}
