// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/ObjectSource/IObjectSelectionSourceModel.h"

namespace UE::ConcertClientSharedSlate
{
	/** Logic for selecting actors from the current editor world. */
	class CONCERTCLIENTSHAREDSLATE_API FActorSelectionSourceModel : public ConcertSharedSlate::IObjectSelectionSourceModel
	{
	public:

		FActorSelectionSourceModel();

		//~ Begin IObjectSelectionSourceModel Interface
		virtual TArray<ConcertSharedSlate::FObjectSourceCategory> GetRootSources() const override;
		virtual TArray<TSharedRef<ConcertSharedSlate::IObjectSourceModel>> GetContextMenuOptions(const FSoftObjectPath& Item) override;
		//~ End IObjectSelectionSourceModel Interface

	private:

		/** Contains actions around adding actor related objects. Displayed next to the search bar. Refined with dynamic options in GetRootSources. */
		ConcertSharedSlate::FObjectSourceCategory BaseActorCategory;
	};
}

