// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Editor/Model/ObjectSource/IObjectSourceModel.h"

namespace UE::ConcertClientSharedSlate
{
	/** Gets the actors currently selected in the editor. */
	class CONCERTCLIENTSHAREDSLATE_API FSelectedActorsSource : public ConcertSharedSlate::IObjectSourceModel
	{
	public:

		//~ Begin IObjectSourceModel Interface
		virtual ConcertSharedSlate::FSourceDisplayInfo GetDisplayInfo() const override;
		virtual uint32 GetNumSelectableItems() const override;
		virtual void EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FSelectableObjectInfo& SelectableOption)> Delegate) const override;
		//~ End IObjectSourceModel Interface
	};
}
