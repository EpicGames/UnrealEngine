// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Model/Item/IItemSourceModel.h"
#include "Replication/Editor/Model/ObjectSource/IObjectSourceModel.h"

namespace UE::ConcertClientSharedSlate
{
	/** Gets all the actors in the current GWorld */
	class CONCERTCLIENTSHAREDSLATE_API FWorldActorSource : public ConcertSharedSlate::IObjectSourceModel
	{
	public:

		//~ Begin IObjectSourceModel Interface
		virtual ConcertSharedSlate::FSourceDisplayInfo GetDisplayInfo() const override;
		virtual uint32 GetNumSelectableItems() const override;
		virtual void EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FSelectableObjectInfo& SelectableOption)> Delegate) const override;
		//~ End IObjectSourceModel Interface
	};
}


