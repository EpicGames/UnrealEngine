// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"
#include "Replication/Editor/Model/Property/IPropertySourceModel.h"

#include "HAL/Platform.h"
#include "Misc/CoreMiscDefines.h"
#include "UObject/WeakObjectPtr.h"

class UClass;

namespace UE::ConcertClientSharedSlate
{
	/** The allowed properties are those returned by UE::ConcertSyncCore::ForEachReplicatableProperty.*/
	class CONCERTCLIENTSHAREDSLATE_API UE_DEPRECATED(5.5, "No longer in use") FConcertSyncCoreReplicatedPropertySource
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		: public ConcertSharedSlate::IPropertySourceModel
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	public:

		void SetClass(UClass* InClass);
		
		//~ Begin IPropertySourceModel Interface
		virtual ConcertSharedSlate::FSourceDisplayInfo GetDisplayInfo() const override;
		virtual uint32 GetNumSelectableItems() const override { return NumProperties; }
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual void EnumerateSelectableItems(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FSelectablePropertyInfo& SelectableOption)> Delegate) const override;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		//~ End IPropertySourceModel Interface

	private:

		/** Weak ptr because it could be a Blueprint class that can be destroyed at any time. */
		TWeakObjectPtr<UClass> Class;

		int32 NumProperties = 0;
	};
}
