// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/PropertyChainUtils.h"
#include "Replication/Editor/Model/Property/IPropertySource.h"

namespace UE::ConcertClientSharedSlate
{
	/** Lists all properties that can be replicated. */
	class FReplicatablePropertySource : public ConcertSharedSlate::IPropertySource
	{
	public:

		explicit FReplicatablePropertySource(UClass* Class)
			: Class(Class)
		{}

		//~ Begin IPropertySource Interface
		virtual void EnumerateProperties(TFunctionRef<EBreakBehavior(const ConcertSharedSlate::FPropertyInfo& Property)> Delegate) const override
		{
			if (!Class.IsValid())
			{
				return;
			}
			
			ConcertSyncCore::PropertyChain::ForEachReplicatableConcertProperty(*Class.Get(), [this, &Delegate](FConcertPropertyChain&& Property)
			{
				return Delegate(ConcertSharedSlate::FPropertyInfo(Property));
			});
		}
		//~ End IPropertySource Interface

	private:
		
		TWeakObjectPtr<UClass> Class;
	};
}
