// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSyncCoreReplicatedPropertySource.h"
#include "Replication/Editor/Model/Property/IPropertySourceProcessor.h"
#include "Templates/SharedPointer.h"

struct FSoftClassPath;

namespace UE::ConcertClientSharedSlate
{
	/**
	 * Decides which properties can be added to IEditableReplicationStreamModel.
	 * The allowed properties are those returned by UE::ConcertSyncCore::ForEachReplicatableProperty.
	 */
	class CONCERTCLIENTSHAREDSLATE_API FSelectPropertyFromUClassModel : public ConcertSharedSlate::IPropertySourceProcessor
	{
	public:

		//~ Begin IPropertySelectionProcessor Interface
		virtual void ProcessPropertySource(
			const ConcertSharedSlate::FPropertySourceContext& Context,
			TFunctionRef<void(const ConcertSharedSlate::IPropertySource& Model)> Processor
			) const override;
		//~ End IPropertySelectionProcessor Interface
	};
}

