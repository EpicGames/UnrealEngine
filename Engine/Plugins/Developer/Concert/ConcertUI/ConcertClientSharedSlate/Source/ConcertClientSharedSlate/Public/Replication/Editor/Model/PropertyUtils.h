// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Replication/Data/ConcertPropertySelection.h"

#include "Containers/ContainersFwd.h"
#include "Containers/ArrayView.h"
#include "Containers/Array.h"
#include "Templates/Function.h"

namespace UE::ConcertClientSharedSlate::PropertyUtils
{
	/**
	 * Called when the user adds a property in the property tree view.
	 * This finds additional properties that should also be added.
	 *
	 * Examples:
	 * - User adds a struct property: Add all child properties automatically
	 * - User adds a container property: All all child properties automatically if it is a struct
	 *
	 * @param ObjectClass The class the properties belong to
	 * @param PropertiesToAdd The properties about to be added a replication stream model
	 * @param Callback Callback to receive the additional properties
	 */
	CONCERTCLIENTSHAREDSLATE_API void EnumerateAdditionalPropertiesToAdd(
		const UClass& ObjectClass,
		TConstArrayView<FConcertPropertyChain> PropertiesToAdd,
		TFunctionRef<void(FConcertPropertyChain&& AdditionalProperty)> Callback
		);

	/** Special version of EnumerateAdditionalPropertiesToAdd that appends any additional properties to the same array. */
	inline void AppendAdditionalPropertiesToAdd(const UClass& ObjectClass, TArray<FConcertPropertyChain>& InOutPropertiesToAdd)
	{
		// No we cannot directly add to InOutPropertiesToAdd in the callback because that may reallocate the memory that the TConstArrayView is reading!
		TArray<FConcertPropertyChain> AdditionalProperties;
		EnumerateAdditionalPropertiesToAdd(ObjectClass, InOutPropertiesToAdd, [&AdditionalProperties](FConcertPropertyChain&& Chain){ AdditionalProperties.Emplace(MoveTemp(Chain)); });
		InOutPropertiesToAdd.Append(AdditionalProperties);
	}

	inline void AppendAdditionalPropertiesToAdd(const FSoftClassPath& ObjectClass, TArray<FConcertPropertyChain>& InOutPropertiesToAdd)
	{
		const UClass* Class = ObjectClass.TryLoadClass<UObject>();
		if (ensureAlwaysMsgf(Class, TEXT("Unresolved class %s"), *ObjectClass.ToString()))
		{
			AppendAdditionalPropertiesToAdd(*Class, InOutPropertiesToAdd);
		}
	}
}
