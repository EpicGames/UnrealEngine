// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Templates/UnrealTemplate.h"

class UWorld;
struct FSoftObjectPath;

namespace UE::ConcertClientSharedSlate
{
	/**
	 * This logic is used to show only those replicated objects that are in the world the user has opened in the editor.
	 * Its functions are injected into the FCreateViewerParams.
	 */
	class CONCERTCLIENTSHAREDSLATE_API FHideObjectsNotInWorldLogic : public FNoncopyable
	{
	public:
		
		FHideObjectsNotInWorldLogic();
		~FHideObjectsNotInWorldLogic();
		
		/** @return Whether this object should be shown to the user. */
		bool ShouldShowObject(const FSoftObjectPath& ObjectPath) const;

		DECLARE_MULTICAST_DELEGATE(FOnRefreshObjects);
		FOnRefreshObjects& OnRefreshObjects() { return OnRefreshObjectsDelegate; }
		
	private:

		/** Broadcasts when the set of displayed objects has changed, e.g. when switching worlds. */
		FOnRefreshObjects OnRefreshObjectsDelegate;
		
		void OnWorldAdded(UWorld* World) const { OnRefreshObjectsDelegate.Broadcast(); }
		void OnWorldDestroyed(UWorld* World) const { OnRefreshObjectsDelegate.Broadcast(); }
	};
}

