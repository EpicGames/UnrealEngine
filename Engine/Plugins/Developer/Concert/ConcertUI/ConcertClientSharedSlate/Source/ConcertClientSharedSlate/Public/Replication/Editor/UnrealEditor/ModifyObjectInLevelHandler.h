// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "Templates/UnrealTemplate.h"

class AActor;
class FTransactionObjectEvent;
class UObject;

namespace UE::ConcertSharedSlate
{
	class IEditableReplicationStreamModel;
	class IReplicationStreamViewer;
}

namespace UE::ConcertClientSharedSlate
{
	/**
	 * Handles updating the passed in model in the following cases:
	 * - Actor removed: Remove from model
	 * - Component or subobject added / removed: refresh hierarchy
	 */
	class CONCERTCLIENTSHAREDSLATE_API FModifyObjectInLevelHandler : public FNoncopyable
	{
	public:
		
		/**
		 * @param UpdatedModel The model to modified when an object is modified. The caller ensures it outlives the constructed object.
		 */
		FModifyObjectInLevelHandler(
			ConcertSharedSlate::IEditableReplicationStreamModel& UpdatedModel UE_LIFETIMEBOUND
			);
		~FModifyObjectInLevelHandler();

		DECLARE_MULTICAST_DELEGATE(FOnHierarchyNeedsRefresh);
		/** Broadcasts when the hierarchy may have changed. */
		FOnHierarchyNeedsRefresh& OnHierarchyNeedsRefresh() { return OnHierarchyNeedsRefreshDelegate; }

	private:

		/** The model that is updated when an object is modified. */
		ConcertSharedSlate::IEditableReplicationStreamModel& UpdatedModel;

		/** Broadcasts when the hierarchy may have changed. */
		FOnHierarchyNeedsRefresh OnHierarchyNeedsRefreshDelegate;
		
		void OnActorDeleted(AActor* Actor) const;
		void OnObjectTransacted(UObject* Object, const FTransactionObjectEvent& TransactionObjectEvent) const;
	};
}
