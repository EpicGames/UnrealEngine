// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/SoftObjectPtr.h"

class FText;

namespace UE::ConcertClientSharedSlate
{
	/** @return The display name of ObjectPath to use for Concert replication UI. */
	CONCERTCLIENTSHAREDSLATE_API FText GetObjectDisplayName(const TSoftObjectPtr<>& ObjectPath);
}

