// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertInheritableClassOption.generated.h"

/** Base struct for settings intended for classes */
USTRUCT()
struct FConcertInheritableClassOption
{
	GENERATED_BODY()

	/** Whether to inherit the settings from the base class. */
	UPROPERTY(EditAnywhere, Category = "Config")
	bool bInheritFromBase = true;
};
