// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertSubobjectMatchingRules.h"
#include "ConcertPerClassSubobjectMatchingRules.generated.h"

USTRUCT()
struct CONCERTCLIENTSHAREDSLATE_API FConcertInheritableSubobjectMatchingRules : public FConcertSubobjectMatchingRules
{
	GENERATED_BODY()

	/** Whether to inherit the settings from the base class. */
	UPROPERTY(EditAnywhere, Category = "Config")
	bool bInheritFromBase = true;
};

/**
 * Binds subobject matching rules to classes.
 * The matching rules can then be applied recursively, if FConcertSubobjectDefaultSelectionRules::bInheritFromBase is set to true.
 *
 * For example, suppose you have set up two sets of rules: one for matching subobjects of an UStaticMeshComponent and one for USkeletalMeshComponent.
 * If the bInheritFromBase is set true for USkeletalMeshComponent's rules, then UStaticMeshComponent will also be applied when searching a skeletal mesh component.
 */
USTRUCT()
struct CONCERTCLIENTSHAREDSLATE_API FConcertPerClassSubobjectMatchingRules
{
	GENERATED_BODY()
	
	/**
	 * Maps matching rules to actor and component classes.
	 * 
	 * Examples:
	 *  - Whenever you add a StaticMeshActor actor, you its static mesh component to be added automatically, too (set IncludeSubobjectRegex to the components name).
	 *  - Whenever you add a component, you want to add its child components, too (set IncludeAllOption = AllComponents) .
	 */
	UPROPERTY(EditAnywhere, Config, Category = "Replication|Editor")
	TMap<FSoftClassPath, FConcertInheritableSubobjectMatchingRules> SubobjectMatchingRules;

	/** Recursively walks up Object class hierarchy and tries to find subobjects */
	void MatchSubobjectsRecursivelyBreakable(const UObject& Object, TFunctionRef<EBreakBehavior(UObject&)> OnSubobjectMatched) const;
	void MatchSubobjectsRecursively(const UObject& Object, TFunctionRef<void(UObject&)> OnSubobjectMatched) const
	{
		return MatchSubobjectsRecursivelyBreakable(Object, [&OnSubobjectMatched](UObject& Object)
		{
			OnSubobjectMatched(Object);
			return EBreakBehavior::Continue;
		});
	}
};