// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertInheritableClassOption.h"
#include "ConcertPerClassSubobjectMatchingRules.h"
#include "Templates/Function.h"
#include "ConcertStreamObjectAutoBindingRules.generated.h"

class UObject;
struct FConcertPropertyChain;
struct FConcertReplicatedObjectInfo;

USTRUCT()
struct CONCERTCLIENTSHAREDSLATE_API FConcertDefaultPropertySelection : public FConcertInheritableClassOption
{
	GENERATED_BODY()

	/**
	 * A list of properties that should be selected by default when you add a new object type.
	 * Specify properties by using the names as the appear in the replication editor and each property with ".".
	 * Example: "RelativeLocation.X"
	 */
	UPROPERTY(EditAnywhere, Category = "Config")
	TArray<FString> DefaultSelectedProperties;
};

/**
 * When a user adds an object via the stream editor, you may want to automatically bind properties and add additional subobjects
 * This structure contains rules to achieve this.
 */
USTRUCT()
struct CONCERTCLIENTSHAREDSLATE_API FConcertStreamObjectAutoBindingRules
{
	GENERATED_BODY()
	
	/** Properties you want selected by default when you add a new replicated object in the editor */
	UPROPERTY(EditAnywhere, Category = "Replication|Editor")
	TMap<FSoftClassPath, FConcertDefaultPropertySelection> DefaultPropertySelection;

	/**
	 * Rules for auto adding additional subobjects.
	 * 
	 * For example, you can set this up so when you add a static mesh actor, its static mesh component is automatically added, too.
	 * The simplest way to achieve this is to bind AStaticMeshActor class to a setting with IncludeAllOption set to AllComponents. 
	 * 
	 * Matched subobjects will be auto added, too.
	 */
	UPROPERTY(EditAnywhere, Category = "Replication|Editor")
	FConcertPerClassSubobjectMatchingRules DefaultAddedSubobjectRules;

	/** Reads DefaultPropertySelection and calls Callback for any default property selections based on the Class just added. */
	void AddDefaultPropertiesFromSettings(UClass& Class, TFunctionRef<void(FConcertPropertyChain&& Chain)> Callback) const;
};