// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/EBreakBehavior.h"
#include "Templates/Function.h"
#include "UObject/SoftObjectPath.h"
#include "ConcertSubobjectMatchingRules.generated.h"

UENUM()
enum class EConcertIncludeAllSubobjectsType : uint8
{
	/** Include nothing by default */
	None,
	/** Include all components */
	AllComponents,
	/** Include all subobjects recursively (includes components, their subobjects, etc. ) */
	AllSubobjects
};

/** Defines rules for finding subobjects of an object. */
USTRUCT()
struct FConcertSubobjectMatchingRules
{
	GENERATED_BODY()
	
	/**
	 * A list of subobject classes to add by default.
	 * For example, when you add an actor, you want to also add certain component classes.
	 */
	UPROPERTY(EditAnywhere, Category = "Config")
	TSet<FSoftClassPath> IncludeClasses;

	/**
	 * Components that match any of this regex will be excluded from IncludeClasses.
	 *
	 * Tips:
	 * - If you want to include "Component" but not "ComponentName" you can use boundaries "\bComponent\b".
	 * . ".*" matches any sequence of characters.
	 */
	UPROPERTY(EditAnywhere, Category = "Config")
	TSet<FString> ExcludeSubobjectRegex;

	/**
	 * Components that match any of this regex will be included as well.
	 * 
	 * Tips:
	 * - If you want to include "Component" but not "ComponentName" you can use boundaries "\bComponent\b".
	 * . ".*" matches any sequence of characters. 
	 */
	UPROPERTY(EditAnywhere, Category = "Config")
	TSet<FString> IncludeSubobjectRegex;

	/** Behaviour to configure for including all of a certain type of subobject. */
	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Config")
	EConcertIncludeAllSubobjectsType IncludeAllOption = EConcertIncludeAllSubobjectsType::None;

	/**
	 * Looks for subobjects in AddedObject that match these rules.
	 * This looks for direct subobjects only though you can call MatchToSubobjectsIn recursively on what you received through OnSubobjectMatched.
	 */
	void MatchToSubobjectsBreakable(const UObject& AddedObject, TFunctionRef<EBreakBehavior(UObject&)> OnSubobjectMatched) const;
	void MatchToSubobjects(const UObject& AddedObject, TFunctionRef<void(UObject&)> OnSubobjectMatched) const
	{
		return MatchToSubobjectsBreakable(AddedObject, [&OnSubobjectMatched](UObject& Object)
		{
			OnSubobjectMatched(Object);
			return EBreakBehavior::Continue;
		});
	}
};
