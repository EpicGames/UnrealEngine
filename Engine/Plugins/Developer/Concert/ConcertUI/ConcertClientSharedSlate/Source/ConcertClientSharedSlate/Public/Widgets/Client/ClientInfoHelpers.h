// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConcertMessageData.h"
#include "IConcertClient.h"
#include "IConcertSession.h"
#include "Widgets/Client/ClientInfoDelegate.h"
#include "Widgets/Client/SClientName.h"

#include "Misc/Attribute.h"
#include "Misc/Optional.h"

/** Util types that can be used with SClientName, etc. */
namespace UE::ConcertClientSharedSlate
{
	inline ConcertSharedSlate::FGetOptionalClientInfo MakeClientInfoGetter(const TSharedRef<IConcertClient>& Client)
	{
		return ConcertSharedSlate::FGetOptionalClientInfo::CreateLambda([WeakClient = Client.ToWeakPtr()](const FGuid& ClientEndpointId) -> TOptional<FConcertClientInfo>
		{
			TSharedPtr<IConcertClient> ClientPin = WeakClient.Pin();
			if (!ClientPin)
			{
				return {};
			}

			const TSharedPtr<IConcertClientSession> Session = ClientPin->GetCurrentSession();
			if (!Session)
			{
				return {};
			}
			
			// FindSessionClient does not work for the local client
			if (ClientEndpointId == Session->GetSessionClientEndpointId())
			{
				return ClientPin->GetClientInfo();
			}
			
			FConcertSessionClientInfo ClientInfo;
			Session->FindSessionClient(ClientEndpointId, ClientInfo);
			return ClientInfo.ClientInfo;
		});
	}
	
	inline ConcertSharedSlate::FIsLocalClient MakeIsLocalClientGetter(const TSharedRef<IConcertClient>& Client)
	{
		return ConcertSharedSlate::FIsLocalClient::CreateLambda([WeakClient = Client.ToWeakPtr()](const FGuid& ClientEndpointId)
		{
			const TSharedPtr<IConcertClient> ClientPin = WeakClient.Pin();
			return ClientPin && ClientPin->GetCurrentSession() && ClientPin->GetCurrentSession()->GetSessionClientEndpointId() == ClientEndpointId;
		});
	}

	/** @return A parentheses delegate that returns "You" if the endpoint ID is that of Client and returns FText::GetEmpty() otherwise. */
	inline ConcertSharedSlate::FGetClientParenthesesContent MakeGetLocalClientParenthesesContent(const TSharedRef<IConcertClient>& Client)
	{
		return ConcertSharedSlate::FGetClientParenthesesContent::CreateLambda([WeakClient = Client.ToWeakPtr()](const FGuid& ClientEndpointId)
		{
			const TSharedPtr<IConcertClient> ClientPin = WeakClient.Pin();
			const bool bIsLocalClient =  ClientPin
				&& ClientPin->GetCurrentSession()
				&& ClientPin->GetCurrentSession()->GetSessionClientEndpointId() == ClientEndpointId;
			return bIsLocalClient ? ConcertSharedSlate::ParenthesesClientNameContent::LocalClient : FText::GetEmpty();
		});
	}

	inline TAttribute<TOptional<FConcertClientInfo>> MakeLocalClientInfoAttribute(const TSharedRef<IConcertClient>& Client)
	{
		return TAttribute<TOptional<FConcertClientInfo>>::CreateLambda([WeakClient = Client.ToWeakPtr()]()
		{
			const TSharedPtr<IConcertClient> ClientPin = WeakClient.Pin();
			return ClientPin ? ClientPin->GetClientInfo() : TOptional<FConcertClientInfo>{};
		});
	}

	inline TAttribute<TOptional<FConcertClientInfo>> MakeClientInfoAttribute(const TSharedRef<IConcertClient>& Client, const FGuid& ClientId)
	{
		return TAttribute<TOptional<FConcertClientInfo>>::CreateLambda([WeakClient = Client.ToWeakPtr(), ClientId]() -> TOptional<FConcertClientInfo>
		{
			const TSharedPtr<IConcertClient> ClientPin = WeakClient.Pin();
			if (!ClientPin)
			{
				return {};
			}

			const TSharedPtr<IConcertClientSession> Session = ClientPin->GetCurrentSession();
			if (!Session)
			{
				return {};
			}

			// FindSessionClient does not work for the local client
			if (ClientId == Session->GetSessionClientEndpointId())
			{
				return ClientPin->GetClientInfo();
			}
			
			FConcertSessionClientInfo Info;
			Session->FindSessionClient(ClientId, Info);
			return Info.ClientInfo;
		});
	}
}